{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "OpenImageDenoise",
	"Description" : "Denoising engine for the Unreal Path Tracer based on Intel's OpenImageDenoise library.",
	"Category" : "Denoising",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "OpenImageDenoise",
			"Type" : "ClientOnly",
			"LoadingPhase" : "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [
				"Win64:arm64",
				"Win64:arm64ec"
			]
		}
	]
}
