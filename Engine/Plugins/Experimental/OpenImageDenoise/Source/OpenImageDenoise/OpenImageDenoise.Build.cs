// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class OpenImageDenoise : ModuleRules
	{
		public OpenImageDenoise(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"RenderCore",
					"Renderer",
					"RHI",
					"IntelOIDN"
				}
			);
			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("MessageLog");
			}
		}
	}
}
