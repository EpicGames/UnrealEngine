// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PathTracingDenoiser.h"
#include "RenderGraphBuilder.h"
#include "RHICommandList.h"
#include "RHIGPUReadback.h"
#include "RHIResources.h"
#include "RHITypes.h"
#include "SceneView.h"

#if WITH_INTELOIDN
#include "OpenImageDenoise/oidn.hpp"

BEGIN_SHADER_PARAMETER_STRUCT(FDenoiseTextureParameters, )
	RDG_TEXTURE_ACCESS(InputTexture, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(InputAlbedo, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(InputNormal, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(OutputTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

class FOpenImageDenoiseModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

#if WITH_EDITOR
DECLARE_LOG_CATEGORY_EXTERN(LogOpenImageDenoise, Log, All);
DEFINE_LOG_CATEGORY(LogOpenImageDenoise);
#endif

IMPLEMENT_MODULE(FOpenImageDenoiseModule, OpenImageDenoise)

static TAutoConsoleVariable<bool> CVarOIDNDenoiseAlpha(
	TEXT("r.OIDN.DenoiseAlpha"),
	true,
	TEXT("Should OpenImageDenoise denoise the alpha channel? (default: true)")
);

static TAutoConsoleVariable<bool> CVarOIDNDenoiseAux(
	TEXT("r.OIDN.DenoiseAuxilaryInputs"),
	false,
	TEXT("Should OpenImageDenoise denoise the auxilary buffers (albedo and normal) prior to using them? (default: false)")
);

static TAutoConsoleVariable<bool> CVarOIDNUseAux(
	TEXT("r.OIDN.UseAuxilaryInputs"),
	true,
	TEXT("Should OpenImageDenoise make use of auxilary buffers (albedo and normal) to improve image quality? (default: true)")
);

struct FColor16
{
	uint16_t r, g, b, a;
};

struct FColor32
{
	float r, g, b, a;
};

struct FDenoiseSettings
{
	bool DenoiseAlpha;
	bool DenoiseAux;
	bool UseAux;

	bool operator==(const FDenoiseSettings& Other) const = default;
	bool operator!=(const FDenoiseSettings& Other) const = default;
};

FDenoiseSettings GetCurrentSettings()
{
	FDenoiseSettings Settings = {};
	Settings.DenoiseAlpha = CVarOIDNDenoiseAlpha.GetValueOnRenderThread();
	Settings.UseAux = CVarOIDNUseAux.GetValueOnRenderThread();
	Settings.DenoiseAux = CVarOIDNDenoiseAux.GetValueOnRenderThread();

	return Settings;
}

struct OIDNState
{
	// scratch CPU memory for running the OIDN filter
	TArray<FColor32> RawPixels;
	TArray<FColor16> RawAlbedo;
	TArray<FColor16> RawNormal;

	// re-useable filters
	oidn::FilterRef AlbedoFilter;
	oidn::FilterRef NormalFilter;
	oidn::FilterRef PixelsFilter;
	oidn::FilterRef AlphaFilter;
	oidn::DeviceRef OIDNDevice;

	FDenoiseSettings CurrentSettings = {};
	FIntPoint CurrentSize = FIntPoint(0, 0);

	void UpdateFilter(FIntPoint Size, FDenoiseSettings Settings)
	{
		int NewSize = Size.X * Size.Y;
		if (RawPixels.Num() != NewSize)
		{
			RawPixels.SetNumUninitialized(NewSize);
			RawAlbedo.SetNumUninitialized(NewSize);
			RawNormal.SetNumUninitialized(NewSize);
			// reset filters so we recreate them to point to the new memory allocation
			AlbedoFilter = oidn::FilterRef();
			NormalFilter = oidn::FilterRef();
			PixelsFilter = oidn::FilterRef();
			AlphaFilter = oidn::FilterRef();
		}
		if (!OIDNDevice)
		{
			OIDNDevice = oidn::newDevice();
			OIDNDevice.commit();
		}
		if (!PixelsFilter || CurrentSettings != Settings || CurrentSize != Size)
		{
#if WITH_EDITOR
			UE_LOGF(LogOpenImageDenoise, Log, "Updating filter configuration for (%d x %d) with denoise_alpha=%s, denoise_aux=%s, use_aux=%s)", Size.X, Size.Y,
				   Settings.DenoiseAlpha ? "on" : "off",
				   Settings.DenoiseAux   ? "on" : "off",
				   Settings.UseAux       ? "on" : "off");
#endif
			CurrentSettings = Settings;
			CurrentSize = Size;
			if (CurrentSettings.DenoiseAux)
			{
				AlbedoFilter = OIDNDevice.newFilter("RT");
				AlbedoFilter.setImage("albedo", RawAlbedo.GetData(), oidn::Format::Half3, Size.X, Size.Y, 0, sizeof(FColor16), sizeof(FColor16) * Size.X);
				AlbedoFilter.setImage("output", RawAlbedo.GetData(), oidn::Format::Half3, Size.X, Size.Y, 0, sizeof(FColor16), sizeof(FColor16) * Size.X);
				AlbedoFilter.set("quality", oidn::Quality::High);
				AlbedoFilter.commit();
				NormalFilter = OIDNDevice.newFilter("RT");
				NormalFilter.setImage("normal", RawNormal.GetData(), oidn::Format::Half3, Size.X, Size.Y, 0, sizeof(FColor16), sizeof(FColor16) * Size.X);
				NormalFilter.setImage("output", RawNormal.GetData(), oidn::Format::Half3, Size.X, Size.Y, 0, sizeof(FColor16), sizeof(FColor16) * Size.X);
				NormalFilter.set("quality", oidn::Quality::High);
				NormalFilter.commit();
			}
			else
			{
				AlbedoFilter = oidn::FilterRef();
				NormalFilter = oidn::FilterRef();
			}
			PixelsFilter = OIDNDevice.newFilter("RT");
			PixelsFilter.setImage("color" , RawPixels.GetData(), oidn::Format::Float3, Size.X, Size.Y, 0, sizeof(FColor32), sizeof(FColor32) * Size.X);
			PixelsFilter.setImage("output", RawPixels.GetData(), oidn::Format::Float3, Size.X, Size.Y, 0, sizeof(FColor32), sizeof(FColor32) * Size.X);

			if (CurrentSettings.UseAux)
			{
				// default behavior, use the albedo/normal buffers to improve quality
				PixelsFilter.setImage("albedo", RawAlbedo.GetData(), oidn::Format::Half3, Size.X, Size.Y, 0, sizeof(FColor16), sizeof(FColor16) * Size.X);
				PixelsFilter.setImage("normal", RawNormal.GetData(), oidn::Format::Half3, Size.X, Size.Y, 0, sizeof(FColor16), sizeof(FColor16) * Size.X);
			}
			if (CurrentSettings.DenoiseAux && CurrentSettings.UseAux)
			{
				// +cleanAux
				PixelsFilter.set("cleanAux", true);
				PixelsFilter.set("quality", oidn::Quality::High);
			}
			PixelsFilter.set("hdr", true);
			PixelsFilter.commit();

			if (CurrentSettings.DenoiseAlpha)
			{
				AlphaFilter = OIDNDevice.newFilter("RT");
				AlphaFilter.setImage("color" , RawPixels.GetData(), oidn::Format::Float, Size.X, Size.Y, sizeof(float) * 3, sizeof(FColor32), sizeof(FColor32) * Size.X);
				AlphaFilter.setImage("output", RawPixels.GetData(), oidn::Format::Float, Size.X, Size.Y, sizeof(float) * 3, sizeof(FColor32), sizeof(FColor32) * Size.X);
				AlphaFilter.set("hdr", true);
				AlphaFilter.commit();
			}
			else
			{
				AlphaFilter = oidn::FilterRef();
			}
		}
	}
};

template <typename PixelType>
static void CopyTextureFromGPUToCPU(FRHICommandListImmediate& RHICmdList, FRHITexture* SrcTexture, FIntPoint Size, TArray<PixelType>& DstArray)
{
	FRHIGPUTextureReadback Readback(TEXT("DenoiserReadback"));
	Readback.EnqueueCopy(RHICmdList, SrcTexture, FIntVector::ZeroValue, 0, FIntVector(Size.X, Size.Y, 1));
	RHICmdList.SubmitAndBlockUntilGPUIdle();

	int32_t SrcStride = 0;
	const PixelType* SrcBuffer = static_cast<PixelType*>(Readback.Lock(SrcStride, nullptr));

	PixelType* DstBuffer = DstArray.GetData();
	for (int Y = 0; Y < Size.Y; Y++, DstBuffer += Size.X, SrcBuffer += SrcStride)
	{
		FPlatformMemory::Memcpy(DstBuffer, SrcBuffer, Size.X * sizeof(PixelType));

	}
	Readback.Unlock();
}

template <typename PixelType>
static void CopyTextureFromCPUToGPU(FRHICommandListImmediate& RHICmdList, const TArray<PixelType>& SrcArray, FIntPoint Size, FRHITexture* DstTexture)
{
	uint32_t DestStride;
	PixelType* DstBuffer = static_cast<PixelType*>(RHICmdList.LockTexture2D(DstTexture, 0, RLM_WriteOnly, DestStride, false));
	DestStride /= sizeof(PixelType);
	const PixelType* SrcBuffer = SrcArray.GetData();
	for (int Y = 0; Y < Size.Y; Y++, SrcBuffer += Size.X, DstBuffer += DestStride)
	{
		FPlatformMemory::Memcpy(DstBuffer, SrcBuffer, Size.X * sizeof(PixelType));
	}
	RHICmdList.UnlockTexture2D(DstTexture, 0, false);
}

static void Denoise(OIDNState& DenoiserState, FRHICommandListImmediate& RHICmdList, FRHITexture* ColorTex, FRHITexture* AlbedoTex, FRHITexture* NormalTex, FRHITexture* OutputTex, FRHIGPUMask GPUMask)
{
	FDenoiseSettings Settings = GetCurrentSettings();

#if WITH_EDITOR
	// NOTE: the time will include the transfer from GPU to CPU which will include waiting for the GPU pipeline to complete
	uint64 FilterExecuteTime = 0;
	FilterExecuteTime -= FPlatformTime::Cycles64();
#endif

	FIntPoint Size = ColorTex->GetSizeXY();
	FIntRect Rect = FIntRect(0, 0, Size.X, Size.Y);

	DenoiserState.UpdateFilter(Size, Settings);
	CopyTextureFromGPUToCPU(RHICmdList, ColorTex, Size, DenoiserState.RawPixels);
	if (Settings.UseAux || Settings.DenoiseAux)
	{
		CopyTextureFromGPUToCPU(RHICmdList, AlbedoTex, Size, DenoiserState.RawAlbedo);
		CopyTextureFromGPUToCPU(RHICmdList, NormalTex, Size, DenoiserState.RawNormal);
	}
	check(DenoiserState.RawPixels.Num() == Size.X * Size.Y);

	if (Settings.DenoiseAux)
	{
		check(DenoiserState.AlbedoFilter);
		check(DenoiserState.NormalFilter);
		DenoiserState.AlbedoFilter.execute();
		DenoiserState.NormalFilter.execute();
	}

	check(DenoiserState.PixelsFilter);
	DenoiserState.PixelsFilter.execute();
	if (Settings.DenoiseAlpha)
	{
		check(DenoiserState.AlphaFilter);
		DenoiserState.AlphaFilter.execute();
	}
	// copy pixels back to GPU (RGB and A are part of the same buffer)
	CopyTextureFromCPUToGPU(RHICmdList, DenoiserState.RawPixels, Size, OutputTex);

#if WITH_EDITOR
	const char* errorMessage;
	if (DenoiserState.OIDNDevice.getError(errorMessage) != oidn::Error::None)
	{
		UE_LOGF(LogOpenImageDenoise, Warning, "Denoiser failed: %s", errorMessage);
		return;
	}

	FilterExecuteTime += FPlatformTime::Cycles64();
	const double FilterExecuteTimeMS = 1000.0 * FPlatformTime::ToSeconds64(FilterExecuteTime);
	UE_LOGF(LogOpenImageDenoise, Log, "Denoised %d x %d pixels in %.2f ms", Size.X, Size.Y, FilterExecuteTimeMS);
#endif
}

using namespace UE::Renderer::Private;

class FOIDNDenoiser : public IPathTracingDenoiser
{
	mutable OIDNState DenoiserState;

public:
	~FOIDNDenoiser() {}

	void AddPasses(FRDGBuilder& GraphBuilder, const FSceneView& View, const FInputs& Inputs) const override
	{
		FDenoiseTextureParameters* DenoiseParameters = GraphBuilder.AllocParameters<FDenoiseTextureParameters>();
		DenoiseParameters->InputTexture = Inputs.ColorTex;
		DenoiseParameters->InputAlbedo = Inputs.AlbedoTex;
		DenoiseParameters->InputNormal = Inputs.NormalTex;
		DenoiseParameters->OutputTexture = Inputs.OutputTex;

		// Need to read GPU mask outside Pass function, as the value is not refreshed inside the pass
		GraphBuilder.AddPass(RDG_EVENT_NAME("OIDN Denoiser Plugin"), DenoiseParameters, ERDGPassFlags::Readback,
			[DenoiseParameters, GPUMask = View.GPUMask, this](FRHICommandListImmediate& RHICmdList)
		{
			Denoise(DenoiserState,
				RHICmdList,
				DenoiseParameters->InputTexture->GetRHI()->GetTexture2D(),
				DenoiseParameters->InputAlbedo->GetRHI()->GetTexture2D(),
				DenoiseParameters->InputNormal->GetRHI()->GetTexture2D(),
				DenoiseParameters->OutputTexture->GetRHI()->GetTexture2D(),
				GPUMask);
		});
	}
};

void FOpenImageDenoiseModule::StartupModule()
{
#if WITH_EDITOR
	UE_LOGF(LogOpenImageDenoise, Log, "OIDN starting up (built with OIDN %s", OIDN_VERSION_STRING);
#endif

	RegisterSpatialDenoiser(MakeUnique<FOIDNDenoiser>(),TEXT("OIDN"));
}

void FOpenImageDenoiseModule::ShutdownModule()
{
#if WITH_EDITOR
	UE_LOGF(LogOpenImageDenoise, Log, "OIDN shutting down");
#endif

	// Release scratch memory and destroy the OIDN device and filters
	UnregisterDenoiser(TEXT("OIDN"));
}

#endif
